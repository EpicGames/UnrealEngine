@echo off
REM Copyright Epic Games, Inc. All Rights Reserved.
REM -----------------------------------------------
REM UnrealAssetStringifyInstall.bat
REM Batch file that runs UnrealAssetStringifyInstall.py
REM -----------------------------------------------

setlocal
set SCRIPT_DIR=%~dp0

REM Normalize path (remove trailing backslash)
if "%SCRIPT_DIR:~-1%"=="\" set SCRIPT_DIR=%SCRIPT_DIR:~0,-1%

set PYTHON_EXE=%SCRIPT_DIR%\..\..\..\..\Binaries\ThirdParty\Python3\Win64\python.exe
set SCRIPT_PATH=%SCRIPT_DIR%\UnrealAssetStringifyInstall.py

"%PYTHON_EXE%" "%SCRIPT_PATH%"
pause

endlocal