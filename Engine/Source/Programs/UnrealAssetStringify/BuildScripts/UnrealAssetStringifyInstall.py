# Copyright Epic Games, Inc. All Rights Reserved.
#
# Script to build UnrealAssetStringify via RunUBT and setup a umap/
# uasset diff assocation in P4V. This script is windows only but could
# be made to work on linux and os x with some effort and testing

import os
import subprocess
import xml.etree.ElementTree as ET
import shutil
from time import sleep
from pathlib import Path
from datetime import datetime

SCRIPT_DIR = Path(__file__).parent
ENGINE_DIR = SCRIPT_DIR.parent.parent.parent.parent

def get_unreal_stringify_path() -> Path:
    # helper for .exe/noextension. I want to query existence so 
    # haven't found away around this platform check:
    exe_name = "UnrealAssetStringify"
    print(os.name)
    if os.name == "nt":
        # On Windows, prefer .exe if it exists
        path = ENGINE_DIR / "Binaries" / "Win64" / f"{exe_name}.exe"
        if path.exists():
            return path
    # Otherwise, fallback to no extension
    return ENGINE_DIR / "Binaries" / exe_name


def build_unreal_asset_stringify():
    ubt_bat = ENGINE_DIR / "Build" / "BatchFiles" / "RunUBT.bat"
    cmd = [str(ubt_bat), "UnrealAssetStringify", "Win64", "Development"]
    print(f"Running: {' '.join(cmd)}")
    subprocess.run(cmd, check=True)
    if not get_unreal_stringify_path().exists():
        print("🚫 UnrealAssetStringify.exe does not exist, despite RunUBT returning 0")
        sys.exit(1)
    print("✅ Compiled UnrealAssetStringify.exe successfully")


def update_p4v_diff_associations():
    """Insert UnrealAssetStringifyP4VDiff associations into P4V ApplicationSettings.xml."""
    user_home = Path.home()
    xml_path = user_home / ".p4qt" / "ApplicationSettings.xml"
    python_path = ENGINE_DIR / "Binaries" / "ThirdParty" / "Python3" / "Win64" / "python.exe"
    diff_script = ENGINE_DIR / "Source" / "Programs" / "UnrealAssetStringify" / "Scripts" / "UnrealAssetStringifyP4VDiff.py"

    if not xml_path.exists():
        print(f"🚫 {xml_path} not found - skipping P4V diff integration.")
        return
    
    # Create backup before modifying
    timestamp = datetime.now().strftime("%Y-%m-%d_%H%M%S")
    backup_path = xml_path.with_name(f"ApplicationSettings_{timestamp}.bak.xml")
    try:
        shutil.copy2(xml_path, backup_path)
        print(f"💾 P4V ApplicationSettings.xml backup created at: {backup_path}")
    except Exception as e:
        print(f"⚠️ Failed to create backup: {e} - associations not updated")
        return

    print(f"Updating P4V ApplicationSettings.xml at: {xml_path}")
    tree = ET.parse(xml_path)
    root = tree.getroot()

    # Find the ApplicationSettings PropertyList
    if root.tag == "PropertyList" and root.get("varName") == "ApplicationSettings":
        app_settings = root
    else:
        app_settings = root.find(".//PropertyList[@varName='ApplicationSettings']")
    if app_settings is None:
        print("🚫 Could not locate <PropertyList varName='ApplicationSettings'> in XML.")
        return

    # Find (or create) the DiffAssociations node
    diff_assocs = app_settings.find(".//Associations[@varName='DiffAssociations']")
    if diff_assocs is None:
        diff_assocs = ET.SubElement(app_settings, "Associations", {"varName": "DiffAssociations"})
        prop_list = ET.SubElement(diff_assocs, "PropertyList", {"IsManaged": "TRUE", "varName": "Associations"})
    else:
        prop_list = diff_assocs.find(".//PropertyList[@varName='Associations']")
        if prop_list is None:
            prop_list = ET.SubElement(diff_assocs, "PropertyList", {"IsManaged": "TRUE", "varName": "Associations"})

    # Remove old uasset/umap associations, if any:
    for assoc in list(prop_list.findall(".//Association")):
        var_name = assoc.attrib.get("varName")
        if var_name in ("uasset", "umap"):
            prop_list.remove(assoc)

    # Add uasset and umap associations w/ UnrealAssetStringifyP4VDiff
    for var in ("uasset", "umap"):
        new_assoc = ET.SubElement(prop_list, "Association", {"varName": var})
        ET.SubElement(new_assoc, "Application").text = str(python_path)
        ET.SubElement(new_assoc, "Arguments").text = f"{diff_script} %1 %2"

    # Write back the updated XML
    tree.write(xml_path, encoding="utf-8", xml_declaration=True)
    print("✅ Updated uasset/umap diff associations in P4V")

def is_p4v_running():
    # this only works on windows, would have to use ps or similar wo/ psutil
    result = subprocess.run("tasklist", capture_output=True, text=True)
    return "p4v.exe" in result.stdout.lower()

def close_p4v():
    # close any running p4v processes, so that we can update ApplicationSettings.xml
    # and not have our change overwritten when those running processes finally exit:
    print("Closing running P4V instances...")
    try:
        subprocess.run(["taskkill", "/IM", "p4v.exe"], check=True, capture_output=True)
    except Exception as e:
        print("⚠️ Unexpected error invoking taskkill (graceful):", e)
        return False

    for _ in range(10):
        if not is_p4v_running():
            return True
        print("Waiting for P4V to close...")
        sleep(1.0)
    
    print("⚠️ P4V Graceful exit failed, forcing P4V to close")
    try:
        subprocess.run(["taskkill", "/F", "/IM", "p4v.exe"], check=False, capture_output=True)
    except Exception as e:
        print("⚠️ Failed to close P4V processes.")
        print(e.stderr.decode() if e.stderr else str(e))
        return False
        
    for _ in range(3):
        if not is_p4v_running():
            return True
        print("Waiting for P4V to close...")
        sleep(1.0)

    return True

def install_unreal_asset_stringify():
    # first, ask the user what they want to do:
    print("This script will build UnrealAssetStringify and (optionally) associate with p4v uasset/umap diff commmands")
    if is_p4v_running():
        print("⚠️ P4V is running - adding UnrealAssetStringifyP4VDiff to config will close P4V")
    add_p4v = input("❔ Add UnrealAssetStringifyP4VDiff to P4V diff tool config? (y/n): ").strip().lower()

    # no more prompts for input after this point, go ahead and run the installer:
    build_unreal_asset_stringify() # builds our helpful tool
    
    # set up p4 file assocations, if we can:
    if add_p4v == "y":
        if is_p4v_running():
            if not close_p4v():
                return
        if is_p4v_running():
            print("⚠️ P4V assocations not updated - P4V could not be closed")
            return
        update_p4v_diff_associations()


if __name__ == "__main__":
    install_unreal_asset_stringify()