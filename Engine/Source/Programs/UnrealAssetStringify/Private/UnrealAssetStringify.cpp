// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonObjectGraph/Stringify.h"
#include "Logging/StructuredLog.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectThreadContext.h"

#include "RequiredProgramMainCPPInclude.h"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealAssetStringify, Log, All);

IMPLEMENT_APPLICATION(UnrealAssetStringify, "UnrealAssetStringify");

static void PrintUsage()
{
const TCHAR* const Usage = TEXT(R"(Usage:
	UnrealAssetStringify [OPTION]... FILE

Options:
	--save, -s         Save the stringified output to Engine\Programs\UnrealAssetStringify\Saved
	                   instead of simply printing it
	--save=PATH, -s=PATH
	                   Save the stringified output to an explicit file path
	--force, -f        Overwrite the output file if it already exists

Example:
	UnrealAssetStringify.exe "D:\Path\To\A.uasset"
	UnrealAssetStringify.exe --save "D:\Path\To\A.uasset"
	UnrealAssetStringify.exe --save="D:\Output\A.json" "D:\Path\To\A.uasset"
	UnrealAssetStringify.exe -s="D:\Output\A.json" -f "D:\Path\To\A.uasset")");
	UE_LOGFMT(LogUnrealAssetStringify, Display, "{0}", Usage);
}

namespace UE::Private
{
	bool TryInferPackageName(FStringView Filename, FString& OutResult);

	enum class EReturnCodes : int32
	{
		Success,
		BadCommandlineArgs,
		MissingFile,
		BadAsset,
		BadPackageName,
		FileAlreadyExists,
		CouldNotCreateResultFile,
		CouldNotWriteResultFile
	};
}

// Returns a best guess package name given a filename that may or may not be
// in an actual project folder. Doesn't bother looking for conventional
// project structure as that would be slow. This means that assets in /Game/
// will be somewhat mangled, but that seems fine to me. Most packages
// now record their location in the project summary:
bool UE::Private::TryInferPackageName(FStringView Filename, FString& OutResult)
{
	OutResult = FPaths::ChangeExtension(FString(Filename), TEXT(""));
	FPaths::NormalizeFilename(OutResult);
	int32 ColonIndex;
	if (OutResult.FindChar(TEXT(':'), ColonIndex))
	{
		OutResult = OutResult.RightChop(ColonIndex + 1);
	}
	// if the filename has '/Content/' in it then assume the preceding folder
	// is the project name, and the sub folder follows the content folder:
	const FString ContentToken = TEXT("/Content/");
	int32 ContentIndex = OutResult.Find(ContentToken, ESearchCase::IgnoreCase, ESearchDir::FromStart);
	if (ContentIndex != INDEX_NONE)
	{
		// Extract the "root folder" before /Content/
		FString RootFolder = OutResult.Left(ContentIndex);
		FPaths::NormalizeDirectoryName(RootFolder);

		// Take the subpath *after* /Content/
		FString SubPath = OutResult.Mid(ContentIndex + ContentToken.Len());

		// Build virtual package root (e.g. /ProjectName/), aka shorten the root:
		FString RootName = FPaths::GetCleanFilename(RootFolder);
		// Recombine the root and subpath:
		FString PackagePath = FString::Printf(TEXT("/%s/%s"), *RootName, *SubPath);

		// Assign result and validate
		OutResult = MoveTemp(PackagePath);
		FPackageName::RegisterMountPoint(OutResult, OutResult);
		check(FPackageName::IsValidLongPackageName(OutResult, true));
		return true;
	}
	else
	{
		// If the filename does not have a /Content/ folder then the best
		// we can do is drop the drive letter:
		FString ErrorMessage;
		if(FPackageName::TryConvertFilenameToLongPackageName(OutResult, OutResult, &ErrorMessage))
		{
			FPackageName::RegisterMountPoint(OutResult, OutResult);
			check(FPackageName::IsValidLongPackageName(OutResult, true));
			return true;
		}
		UE_LOGFMT(LogUnrealAssetStringify, Error, "Failed to infer package name: {0}", ErrorMessage);
	}
	return false;
}

struct FUnrealAssetStringifyArgs
{
	// options:
	bool bSave = false;
	FString SavePath;   // explicit output path; empty means use default save dir
	bool bForce = false;
	// required inputs:
	FString Filename;

	static bool ParseArgs(int32 ArgC, TCHAR* const ArgV[], FUnrealAssetStringifyArgs& OutArgs);
	static int32 StringifyFile(FStringView Filename, const FUnrealAssetStringifyArgs& InArgs);
};

bool FUnrealAssetStringifyArgs::ParseArgs(int32 ArgC, TCHAR* const ArgV[], FUnrealAssetStringifyArgs& OutArgs)
{
	// Helper: check if Arg matches a flag name with an optional =VALUE suffix.
	// Returns true when the flag matches; if a value is present it is placed in OutValue.
	auto TryParseFlag = [](FStringView Arg, FStringView Flag, FString& OutValue) -> bool
	{
		if(!Arg.StartsWith(Flag))
		{
			return false;
		}
		const FStringView Remainder = Arg.Mid(Flag.Len());
		if(Remainder.IsEmpty())
		{
			OutValue = TEXT("");
			return true;
		}
		if(Remainder[0] == TEXT('='))
		{
			OutValue = FString(Remainder.Mid(1));
			return true;
		}
		return false; // e.g. "--saves" should not match "--save"
	};

	int32 I;
	for(I = 1; I < ArgC - 1; ++I)
	{
		const FStringView Arg(ArgV[I]);
		FString Value;
		if(TryParseFlag(Arg, TEXT("--save"), Value) || TryParseFlag(Arg, TEXT("-s"), Value))
		{
			OutArgs.bSave = true;
			OutArgs.SavePath = MoveTemp(Value);
		}
		else if(Arg == TEXT("--force") || Arg == TEXT("-f"))
		{
			OutArgs.bForce = true;
		}
	}

	if(I < ArgC && !FStringView(ArgV[I]).StartsWith(TEXT("-")))
	{
		OutArgs.Filename = ArgV[I];
	}
	return !OutArgs.Filename.IsEmpty();
}

int32 FUnrealAssetStringifyArgs::StringifyFile(FStringView Filename, const FUnrealAssetStringifyArgs& InArgs)
{
	FPackagePath PackagePath = FPackagePath::FromLocalPath(Filename);
	UPackage* Package = CreatePackage(*PackagePath.GetPackageName());
	check(Package);
	{
		if(!IFileManager::Get().FileExists(*FString(Filename)))
		{
			UE_LOGFMT(LogUnrealAssetStringify, Error, "Could not find file: \"{0}\"", Filename);
			return (int32)UE::Private::EReturnCodes::MissingFile;
		}

		TRefCountPtr<FUObjectSerializeContext> LoadContext(FUObjectThreadContext::Get().GetSerializeContext());
		BeginLoad(LoadContext);
		FLinkerLoad* Linker = FLinkerLoad::CreateLinker(LoadContext, Package, PackagePath, LOAD_NoVerify|LOAD_DisableEngineVersionChecks);
		if(!Linker)
		{
			UE_LOGFMT(LogUnrealAssetStringify, Error, "Could not read file as a uasset: {0}", Filename);
			return (int32)UE::Private::EReturnCodes::BadAsset;
		}
		if((Linker->Summary.GetPackageFlags() & PKG_Cooked) != 0)
		{
			// we can't load cooked packages yet because the cooked format is not interpretable
			// without the native code:
			UE_LOGFMT(LogUnrealAssetStringify, Warning, "Cannot load {0} because it has been cooked and does not have tagged data", Filename);
			return (int32)UE::Private::EReturnCodes::Success;
		}
		// GetPackageName isn't going to work for us, but we can rename the package to the one
		// stored in the asset, which will match user expectation:
		if(Linker->Summary.PackageName != FName())
		{
			Package->Rename(*Linker->Summary.PackageName);
		}
		else
		{
			// The package predates the inclusion of the packagename in the summary
			// lets try to infer a reasonable package name. It cannot efficiently
			// determine /Game/, so there will be quirks.
			FString InferredPackageName;
			if(UE::Private::TryInferPackageName(Filename, InferredPackageName))
			{
				Package->Rename(*InferredPackageName);
			}
			else
			{
				UE_LOGFMT(LogUnrealAssetStringify, Error, 
					"Cannot infer package name for {0} consider resaving the asset to add an explicit package name", Filename);
				return (int32)UE::Private::EReturnCodes::BadPackageName;
			}
		}
		check(Linker);
		Linker->LoadAllObjects(false);
		EndLoad(LoadContext);
	}

	FUtf8String Json = UE::JsonObjectGraph::Stringify(
		{Package}
	);

	// output, either to console or to a file:
	if(!InArgs.bSave)
	{
		UE_LOGFMT(LogUnrealAssetStringify, Display, "Asset printed by UnrealAssetStringify\n{0}", Json);
	}
	else
	{
		FString DestinationFile;
		if(!InArgs.SavePath.IsEmpty())
		{
			// Explicit path supplied via --save=PATH or -s=PATH
			DestinationFile = InArgs.SavePath;
			FPaths::NormalizeFilename(DestinationFile);
		}
		else
		{
			// Default path: Saved/<package-path>_uas.json
			// _uas is a fingerprint for UnrealAssetStringify, .json is the extension
			DestinationFile =
				FPaths::ProjectSavedDir() +
				Package->GetPathName() +
				FString(TEXT("_uas.json"));
		}

		if(IFileManager::Get().FileExists(*DestinationFile) && !InArgs.bForce)
		{
			UE_LOGFMT(LogUnrealAssetStringify, Error,
				"Output file already exists: {0} (use --force / -f to overwrite)", DestinationFile);
			return (int32)UE::Private::EReturnCodes::FileAlreadyExists;
		}

		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*DestinationFile));
		if(!FileArchive) // failed to create a file
		{
			UE_LOGFMT(LogUnrealAssetStringify, Error,
				"Failed to create result file {0}", DestinationFile);
			return (int32)UE::Private::EReturnCodes::CouldNotCreateResultFile;
		}

		FileArchive->Serialize((void*)Json.GetCharArray().GetData(), Json.GetCharArray().Num() - 1);
		if(FileArchive->IsError()) // error during writing, likely disk space
		{
			UE_LOGFMT(LogUnrealAssetStringify, Error,
				"Failed to write result file {0}", DestinationFile);
			return (int32)UE::Private::EReturnCodes::CouldNotWriteResultFile;
		}
		UE_LOGFMT(LogUnrealAssetStringify, Display,
			"Wrote result file: {0}", DestinationFile);
	}
	return (int32)UE::Private::EReturnCodes::Success;
}

FString GetIniBootstrapFilename()
{
	FCommandLine::Set(TEXT(""));
	static FString VersionString = TEXT("uas_delme");
	static bool bVersionStringInialized = false;
	if(!bVersionStringInialized)
	{
		// We use the exe to invalidate cached inis:
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateFileReader(
			FPlatformProcess::ExecutablePath()));
		if (FileArchive)
		{
			TArray<uint8> FileData;
			FileData.SetNum(FileArchive->TotalSize());
			FileArchive->Serialize(FileData.GetData(), FileData.Num());

			FBlake3 Hash;
			Hash.Update(FileData.GetData(), FileData.Num());
			VersionString = LexToString(Hash.Finalize());
		}
	}
	return FString::Printf(TEXT("%s/Bootstrap-UnrealAssetStringify-%s.inis"), *FPaths::ProjectSavedDir(), *VersionString);
}

FString GetIniBootstrap()
{
	// if a cached version of the init exists return that:
	FString BootstrapFilename = GetIniBootstrapFilename();
	if(IFileManager::Get().FileExists(*BootstrapFilename))
	{
		return FString::Printf(TEXT(" -IniBootstrap=\"%s\""), *BootstrapFilename);
	}
	return "";
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	ON_SCOPE_EXIT
	{ 
		FIoDispatcher::Shutdown(); 
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	// Ini boot strap is a fairly critical optimization for standalone programs
	// without this large multiprocess workloads will be much slower, which 
	// inhibits testing:
	FString IniBootStrap = GetIniBootstrap();

	// silence some logs - LogEnum support could be improved,
	// NoPreviewPlatforms is a config optimization
	FString InitCommandLine = 
		TEXT(" -NODEFAULTLOG -LogCmds=\"LogConfig warning, LogLinker error, LogStreaming error, LogUObjectGlobals error, LogEnum error, LogExit error\" -NoPreviewPlatforms") + 
		IniBootStrap;
	FCommandLine::Set(*InitCommandLine);
	FLogSuppressionInterface::Get().ProcessConfigAndCommandLine();
	// disable localization display string support, it is expensive to initialize:
	IConsoleManager::Get().ProcessUserConsoleInput(TEXT("Localization.DisplayStringSupport 2"), *GLog, nullptr);
	// ensure all needed ido and placeholder features are enabled
	IConsoleManager::Get().ProcessUserConsoleInput(TEXT("IDO.Enable 1"), *GLog, nullptr);
	IConsoleManager::Get().ProcessUserConsoleInput(TEXT("IDO.Placeholder.Enable 1"), *GLog, nullptr);
	IConsoleManager::Get().ProcessUserConsoleInput(TEXT("IDO.Placeholder.Feature.ReplaceMissingTypeImportsOnLoad 1"), *GLog, nullptr);
	IConsoleManager::Get().ProcessUserConsoleInput(TEXT("IDO.Placeholder.Feature.SerializeExportReferencesOnLoad 1"), *GLog, nullptr);
	IConsoleManager::Get().ProcessUserConsoleInput(TEXT("IDO.Placeholder.Feature.ReplaceMissingReinstancedTypes 1"), *GLog, nullptr);
	IConsoleManager::Get().ProcessUserConsoleInput(TEXT("IDO.Placeholder.Feature.ReplaceDeadClassInstanceTypes 1"), *GLog, nullptr);
	GAllowUnversionedContentInEditor = true;
	GAllowCookedDataInEditorBuilds = true;
	FScopedAllowAbstractClassAllocation AllowAbstract; // mostly for UMulticastDelegateProperty

	if (int32 Ret = GEngineLoop.PreInit(*InitCommandLine))
	{
		UE_LOGFMT(LogUnrealAssetStringify, Error, "Failed to PreInit");
		return Ret;
	}

	if(IniBootStrap.IsEmpty())
	{
		GConfig->SaveCurrentStateForBootstrap(*GetIniBootstrapFilename());
	}

	FUnrealAssetStringifyArgs Args;
	if(!FUnrealAssetStringifyArgs::ParseArgs(ArgC, ArgV, Args))
	{
		PrintUsage();
		UE_LOGFMT(LogUnrealAssetStringify, Error, "Failed to parse arguments");
		return (int32)UE::Private::EReturnCodes::BadCommandlineArgs;
	}

	// These are initialized lazily, but with no other modules loaded 
	// they won't be initialized at all
	FDelegateProperty::StaticClass();
	FSoftClassProperty::StaticClass();
	FInt8Property::StaticClass();
	FInt16Property::StaticClass();
	FLazyObjectProperty::StaticClass();
	FMulticastDelegateProperty::StaticClass();
	FMulticastInlineDelegateProperty::StaticClass();
	FMulticastSparseDelegateProperty::StaticClass();
	FFieldPathProperty::StaticClass();
	
	return FUnrealAssetStringifyArgs::StringifyFile(Args.Filename, Args);
}
