@echo off
REM Copyright Epic Games, Inc. All Rights Reserved.
REM -----------------------------------------------
REM UnrealAssetStringifyOpenAsset.bat
REM Batch file so that UnrealAssetStringifyOpenAsset.py can be run 
REM via explorer (e.g. Open With). If you need a change in behavior
REM here consider making it in the .py. Batch is difficult to maintain
REM -----------------------------------------------

setlocal
set SCRIPT_DIR=%~dp0

REM Normalize path (remove trailing backslash)
if "%SCRIPT_DIR:~-1%"=="\" set SCRIPT_DIR=%SCRIPT_DIR:~0,-1%

REM relative addressing of python and UnrealAssetStringifyOpenAsset.py:
set PYTHON_EXE=%SCRIPT_DIR%\..\..\..\..\Binaries\ThirdParty\Python3\Win64\python.exe
set SCRIPT_PATH=%SCRIPT_DIR%\UnrealAssetStringifyOpenAsset.py

REM Forward all arguments to Python
start "" /min "%PYTHON_EXE%" "%SCRIPT_PATH%" %*

endlocal