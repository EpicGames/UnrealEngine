# Copyright Epic Games, Inc. All Rights Reserved.

import subprocess
import tempfile
import os
import sys
import shutil
from pathlib import Path

SCRIPT_DIR = Path(__file__).parent
UNREAL_BIN_DIR = SCRIPT_DIR.parent.parent.parent.parent / "Binaries" / "Win64"
UNREAL_STRINGIFY = SCRIPT_DIR / UNREAL_BIN_DIR / "UnrealAssetStringify"

EDITOR = "code"

def run_unreal_asset_stringify(asset_path):
    cmd = [str(UNREAL_STRINGIFY), asset_path]
    print(f"Running: {' '.join(cmd)}")

    try:
        result = subprocess.run(cmd, capture_output=True, text=True)
        json_output = result.stdout.strip()
    except FileNotFoundError:
        json_output = f"Error: UnrealAssetStringify not found at {asset_path} - do you want to remove your file association?"

    if not json_output:
        json_output = "Error: No output received from UnrealAssetStringify."

    temp_file = tempfile.NamedTemporaryFile(delete=False, suffix=".json")
    temp_file.write(json_output.encode("utf-8"))
    temp_file.close()

    print(f"JSON written to: {temp_file.name}")
    return temp_file.name


def open_in_vscode(file_path):
    vscode_cmd = shutil.which(EDITOR)
    if not vscode_cmd:
        print(f"Error: {EDITOR} command not found in PATH.")
        sys.exit(1)

    subprocess.run([vscode_cmd, file_path])


if __name__ == "__main__":
    if len(sys.argv) < 2:
        print("Usage: python UnrealAssetStringifyOpenAsset.py <AssetPath>")
        sys.exit(1)

    asset_path = sys.argv[1]
    json_file = run_unreal_asset_stringify(asset_path)
    open_in_vscode(json_file)