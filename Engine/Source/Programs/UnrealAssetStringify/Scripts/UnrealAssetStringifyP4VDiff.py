# Copyright Epic Games, Inc. All Rights Reserved.
#
# This script prints .uasset files to stdout and displays them
# in the p4merge diff program, if available. It is meant to be
# integrated as custom diff tool in p4v
#
# To enable this tool edit C:\Users\<username>\.p4qt\ApplicationSettings.xml:
r'''
<Associations varName="DiffAssociations">
 <RunExternal>false</RunExternal>
 <PropertyList IsManaged="TRUE" varName="Associations">
  <Association varName="uasset">
   <Application>D:\path\to\Engine\Binaries\ThirdParty\Python3\Win64\python.exe</Application>
   <Arguments>D:\path\to\Engine\Source\Programs\UnrealAssetStringify\Scripts\UnrealAssetStringifyP4VDiff.py %1 %2</Arguments>
  </Association>
  <Association varName="umap">
   <Application>D:\path\to\Engine\Binaries\ThirdParty\Python3\Win64\python.exe</Application>
   <Arguments>D:\path\to\Engine\Source\Programs\UnrealAssetStringify\Scripts\UnrealAssetStringifyP4VDiff.py %1 %2</Arguments>
  </Association>
 </PropertyList>
</Associations>

in <PropertyList IsManaged="TRUE" varName="ApplicationSettings"></PropertyList>
'''
# There is a windows only install script in ..BuildScripts/UnrealAssetStringifyInstall.py

import os
import sys
import tempfile
import subprocess
from pathlib import Path

SCRIPT_DIR = Path(__file__).parent
UNREAL_BIN_DIR = SCRIPT_DIR.parent.parent.parent.parent / "Binaries" / "Win64"

def get_unreal_stringify_path() -> Path:
    # helper for .exe/noextension. I want to query existence so 
    # haven't found away around this platform check:
    exe_name = "UnrealAssetStringify"
    if os.name == "nt":
        # On Windows, prefer .exe if it exists
        path = UNREAL_BIN_DIR / f"{exe_name}.exe"
        if path.exists():
            return path
    # Otherwise, fallback to no extension
    return UNREAL_BIN_DIR / exe_name

UNREAL_STRINGIFY = get_unreal_stringify_path()
P4MERGE = "p4merge"

def write_src_as_json_to_dst(exe: Path, src: Path, dst: Path) -> None:
    with open(dst, "wb") as out:
        proc = subprocess.run([str(exe), str(src)], stdout=out, stderr=out)

def main(argv: list[str]) -> int:
    if len(argv) < 3:
        print("Usage: python UnrealAssetStringifyP4VDiff.py <left> <right>", file=sys.stderr)
        return 1

    left_in, right_in = map(Path, argv[1:3])
    tmp_dir = Path(tempfile.mkdtemp(prefix="p4vjsondiff_"))
    ljson, rjson = tmp_dir / "left.json", tmp_dir / "right.json"

    exe_exists = UNREAL_STRINGIFY.exists()
    exe_missing_not_exists = (
        "UnrealAssetStringify not found.\n"
        "Please build it by running:\n"
        ">RunUBT.bat UnrealAssetStringify Win64 Development\n"
        "From your Engine root directory"
    )
    error_msg_left = ""
    error_msg_right = ""
    if exe_exists:
        write_src_as_json_to_dst(UNREAL_STRINGIFY, left_in, ljson)
        write_src_as_json_to_dst(UNREAL_STRINGIFY, right_in, rjson)
    else:
        error_msg_left = exe_missing_not_exists
        error_msg_right = exe_missing_not_exists

    if error_msg_left != "":
        ljson.write_text(error_msg_left, encoding="utf-8")
    if error_msg_right != "":
        rjson.write_text(error_msg_right, encoding="utf-8")

    return subprocess.call([P4MERGE, str(ljson), str(rjson)])

if __name__ == "__main__":
    sys.exit(main(sys.argv))