// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealAssetStringify : ModuleRules
{
	public UnrealAssetStringify(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Projects", // required for project manager..
				"JsonObjectGraph"
			}
		);
		
		bEnableExceptions = true;
	}
}
