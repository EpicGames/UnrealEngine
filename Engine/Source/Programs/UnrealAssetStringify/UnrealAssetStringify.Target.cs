// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UnrealAssetStringifyTarget : TargetRules
{
	public UnrealAssetStringifyTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "UnrealAssetStringify";
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		// Linking Engine is not viable - would bring in 
		// a huge amount of code:
		bCompileAgainstEngine = false;
		bBuildDeveloperTools = false;
		bCompileAgainstApplicationCore = false;
		bBuildWithEditorOnlyData = true;
		bCompileAgainstEditor = true; // this defines WITH_EDITOR, allowing us to load editor data
		bCompileAgainstCoreUObject = true;

		bIsBuildingConsoleApplication = true;
		bEnableTrace = true;

		// We need full package paths.. making these explicit...
		GlobalDefinitions.Add("UE_SUPPORT_FULL_PACKAGEPATH=1");
		GlobalDefinitions.Add("ALLOW_OTHER_PLATFORM_CONFIG=1"); // required by IConsoleManager.h when using bCompileAgainstEditor
		GlobalDefinitions.Add("UE_INTERNAL_UNIVERSAL_PLACEHOLDERS=1");
		// Disable crashreporter to improve startup time and prevent ini pollution
		GlobalDefinitions.Add("NOINITCRASHREPORTER=1");
		// Another optimization, limit the number of threads:
		GlobalDefinitions.Add("UE_TASKGRAPH_THREAD_LIMIT=5");

		// Late resolve doesn't quite work atm due to CDOs being created
		// immediately by the linker if there's a Class in the asset:
		GlobalDefinitions.Add("UE_WITH_OBJECT_HANDLE_LATE_RESOLVE=0");
	}
}
